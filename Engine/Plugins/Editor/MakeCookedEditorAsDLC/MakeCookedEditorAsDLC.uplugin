{
	"FileVersion": 3,
	"Description": "This plugin is used in conjunction with the MakeCookedEditor UAT script to generate a cooked editor as a DLC add-on to a cooked client. It does not need to be enabled.",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"CanContainContent": true,
	"ExplicitlyLoaded": false,
	"EnabledByDefault": false,
	"EditorOnly": true
}